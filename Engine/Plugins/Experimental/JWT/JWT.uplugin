{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "JSON Web Token Plugin",
	"Description": "An API for working with JSON Web Token (JWT) data.",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "JWT",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Android",
				"IOS",
				"Mac",
				"Win64",
				"Linux",
				"LinuxArm64"
			],
			"PlatformDenyList": [
				"TVOS"
			]
		}
	],
	"Plugins": [
		{
			"Name": "PlatformCrypto",
			"Enabled": true
		}
	]
}
