// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class JWT : ModuleRules
	{
		public JWT(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Json",
					"PlatformCrypto",
					"PlatformCryptoContext"
				}
			);
		}
	}
}
