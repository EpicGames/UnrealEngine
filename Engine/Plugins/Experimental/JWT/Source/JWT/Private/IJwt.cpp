// Copyright Epic Games, Inc. All Rights Reserved.

#include "IJwt.h"
#include "JsonWebToken.h"
#include "JwtGlobals.h"

DEFINE_LOG_CATEGORY(LogJwt);

class FJwt : public IJwt
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FJwt, JWT)



void FJwt::StartupModule()
{
	// This code will execute after your module is loaded into memory (but after global variables are initialized, of course.)
}


void FJwt::ShutdownModule()
{
	// This function may be called during shutdown to clean up your module.  For modules that support dynamic reloading,
	// we call this function before unloading the module.
}


namespace UE::JWT
{
	TOptional<FJsonWebToken> FromString(const FStringView InEncodedJsonWebToken)
	{
		return FJsonWebToken::FromString(InEncodedJsonWebToken);
	}


	bool FromString(const FStringView InEncodedJsonWebToken, FJsonWebToken& OutJsonWebToken)
	{
		return FJsonWebToken::FromString(InEncodedJsonWebToken, OutJsonWebToken);
	}
}
