// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonWebToken.h"

#include "JwtGlobals.h"
#include "JwtUtils.h"
#include "JwtAlgorithms.h"

#include "Algo/Count.h"
#include "Serialization/JsonSerializer.h"


// JWT header field names
const TCHAR *const FJsonWebToken::HEADER_TYPE = TEXT("typ");
const TCHAR *const FJsonWebToken::HEADER_KEY_ID = TEXT("kid");
const TCHAR *const FJsonWebToken::HEADER_ALGORITHM = TEXT("alg");

// JWT header expected values
const TCHAR *const FJsonWebToken::TYPE_VALUE_JWT = TEXT("JWT");

// JWT payload claim field names
const TCHAR *const FJsonWebToken::CLAIM_ISSUER = TEXT("iss");
const TCHAR *const FJsonWebToken::CLAIM_ISSUED_AT = TEXT("iat");
const TCHAR *const FJsonWebToken::CLAIM_EXPIRATION = TEXT("exp");
const TCHAR* const FJsonWebToken::CLAIM_NOT_BEFORE = TEXT("nbf");
const TCHAR *const FJsonWebToken::CLAIM_SUBJECT = TEXT("sub");
const TCHAR *const FJsonWebToken::CLAIM_AUDIENCE = TEXT("aud");


void FJsonWebToken::DumpJsonObject(const FJsonObject& InJsonObject)
{
	if (!UE_LOG_ACTIVE(LogJwt, VeryVerbose))
	{
		return;
	}

	UE_LOGF(LogJwt, VeryVerbose, "size=%d, fields:", InJsonObject.Values.Num());

	for (const TMap<FJsonObject::FStringType, TSharedPtr<FJsonValue>>::ElementType& Pair : InJsonObject.Values)
	{
		const FJsonObject::FStringType& Name = Pair.Key;
		const TSharedPtr<FJsonValue>& Value = Pair.Value;

		if (!Value)
		{
			UE_LOGF(LogJwt, VeryVerbose, "  %ls [undefined]", *Name);
			continue;
		}

		if (Value->IsNull())
		{
			UE_LOGF(LogJwt, VeryVerbose, "  %ls [null]", *Name);
			continue;
		}

		// Present the JSON value in a printable form; TryGetString() is overridden for
		// non-compound types (bool, number, string).
		FString OutString;
		if (Value->TryGetString(OutString))
		{
			UE_LOGF(LogJwt, VeryVerbose, "  %ls: %ls", *Name, *OutString);
			continue;
		}

		const TArray< TSharedPtr<FJsonValue> >* OutArray = nullptr;
		if (Value->TryGetArray(OutArray))
		{
			UE_LOGF(LogJwt, VeryVerbose, "  %ls [array, len=%d]", *Name, OutArray->Num());
			continue;
		}

		const TSharedPtr<FJsonObject>* OutObject = nullptr;
		if (Value->TryGetObject(OutObject))
		{
			const int32 Length = (*OutObject).IsValid() ? (*OutObject)->Values.Num() : 0;

			UE_LOGF(LogJwt, VeryVerbose, "  %ls [object, len=%d]", *Name, Length);
			continue;
		}

		UE_LOGF(LogJwt, VeryVerbose, "  %ls [unknown type])", *Name);
	}
}


TSharedPtr<FJsonObject> FJsonWebToken::FromJson(const FString& InJsonStr)
{
	TSharedRef<TJsonReader<> > Reader = TJsonReaderFactory<>::Create(InJsonStr);

	TSharedPtr<FJsonObject> JsonObject;
	const bool bDeserializeOk = FJsonSerializer::Deserialize(Reader, JsonObject);

	if (!bDeserializeOk)
	{
		UE_LOGF(LogJwt, VeryVerbose, "deserialization of JWT failed");
	}
	else if (!JsonObject.IsValid())
	{
		UE_LOGF(LogJwt, VeryVerbose, "no custom properties in JWT");
	}
	else
	{
		DumpJsonObject(*JsonObject.Get());
	}

	return JsonObject;
}

TSharedPtr<FJsonObject> FJsonWebToken::ParseEncodedJson(const FStringView InEncodedJson)
{
	TSharedPtr<FJsonObject> ParsedObj;

	FString DecodedJson;
	if (FJwtUtils::Base64UrlDecode(InEncodedJson, DecodedJson))
	{
		ParsedObj = FromJson(DecodedJson);
	}

	return ParsedObj;
}

bool FJsonWebToken::FromString(const FStringView InEncodedJsonWebToken, FJsonWebToken& OutJsonWebToken)
{
	TOptional<FJsonWebToken> JsonWebToken = FromString(InEncodedJsonWebToken);

	if (!JsonWebToken.IsSet())
	{
		return false;
	}

	OutJsonWebToken = MoveTemp(JsonWebToken.GetValue());
	return true;
}

TOptional<FJsonWebToken> FJsonWebToken::FromString(const FStringView InEncodedJsonWebToken)
{
	// Check for the correct number of dots.
	if (Algo::Count(InEncodedJsonWebToken, TEXT('.')) != 2)
	{
		UE_LOGF(LogJwt, Verbose, "[FJsonWebToken::FromString] Failed. Token is not in {header}.{payload}.[{signature}] form.");
		return {};
	}

	// Split out the token string into header, payload, and signature parts.
	FStringView EncodedHeader;
	FStringView EncodedPayload;
	FStringView SignaturePart;
	if (!FJwtUtils::SplitEncodedJsonWebTokenString(InEncodedJsonWebToken, EncodedHeader, EncodedPayload, SignaturePart))
	{
		return {};
	}

	// Store the encoded header and payload for signature validation
	FString EncodedHeaderPayload = FString::Printf(TEXT("%s.%s"),
		*FString(EncodedHeader), *FString(EncodedPayload));

	// Decode and parse the header.
	UE_LOGF(LogJwt, VeryVerbose, "[FJsonWebToken::FromString] Parsing JWT header.");
	const TSharedPtr<FJsonObject> HeaderPtr = ParseEncodedJson(EncodedHeader);
	if (!HeaderPtr)
	{
		UE_LOGF(LogJwt, Verbose, "[FJsonWebToken::FromString] Failed to decode and parse the header.");
		return {};
	}

	// Decode and parse the payload.
	UE_LOGF(LogJwt, VeryVerbose, "[FJsonWebToken::FromString] Parsing JWT payload.");
	const TSharedPtr<FJsonObject> PayloadPtr = ParseEncodedJson(EncodedPayload);
	if (!PayloadPtr)
	{
		UE_LOGF(LogJwt, Verbose, "[FJsonWebToken::FromString] Failed to decode and parse the payload.");
		return {};
	}

	// Decode (but do not parse) the signature if it not empty.
	TOptional<TArray<uint8>> Signature;

	if (!SignaturePart.IsEmpty())
	{
		UE_LOGF(LogJwt, VeryVerbose, "[FJsonWebToken::FromString] Decoding JWT signature.");

		TArray<uint8> SignatureBytes;

		if (!FJwtUtils::Base64UrlDecode(SignaturePart, SignatureBytes))
		{
			UE_LOGF(LogJwt, Verbose, "[FJsonWebToken::FromString] Failed to decode the signature.");
			return {};
		}

		Signature = MoveTemp(SignatureBytes);
	}

	const FJsonWebToken JsonWebToken(EncodedHeaderPayload,
		HeaderPtr.ToSharedRef(), PayloadPtr.ToSharedRef(), Signature);

	// Validate the type.  If it exists but is not a JWT, then fail.
	FString Type;
	if (JsonWebToken.GetType(Type) && (Type != TYPE_VALUE_JWT))
	{
		UE_LOGF(LogJwt, Verbose, "[FJsonWebToken::FromString] Failed. Type field exists but is not JWT: \"%ls\"", *Type);
		return {};
	}

	return JsonWebToken;
}


FJsonWebToken::FJsonWebToken(
	const FStringView InEncodedHeaderPayload,
	const TSharedRef<FJsonObject>& InHeaderPtr,
	const TSharedRef<FJsonObject>& InPayloadPtr,
	const TOptional<TArray<uint8>>& InSignature
)
	: EncodedHeaderPayload(InEncodedHeaderPayload)
	, Header(InHeaderPtr)
	, Payload(InPayloadPtr)
	, Signature(InSignature)
{
}

bool FJsonWebToken::GetType(FString& OutValue) const
{
	return Header->TryGetStringField(HEADER_TYPE, OutValue);
}

bool FJsonWebToken::GetKeyId(FString& OutValue) const
{
	return Header->TryGetStringField(HEADER_KEY_ID, OutValue);
}

bool FJsonWebToken::GetAlgorithm(FString& OutValue) const
{
	return Header->TryGetStringField(HEADER_ALGORITHM, OutValue);
}

bool FJsonWebToken::GetIssuer(FString& OutValue) const
{
	return Payload->TryGetStringField(CLAIM_ISSUER, OutValue);
}

bool FJsonWebToken::GetIssuedAt(int64& OutValue) const
{
	return Payload->TryGetNumberField(CLAIM_ISSUED_AT, OutValue);
}

bool FJsonWebToken::GetExpiration(int64& OutValue) const
{
	return Payload->TryGetNumberField(CLAIM_EXPIRATION, OutValue);
}

bool FJsonWebToken::GetNotBefore(int64& OutValue) const
{
	return Payload->TryGetNumberField(CLAIM_NOT_BEFORE, OutValue);
}

bool FJsonWebToken::GetSubject(FString& OutValue) const
{
	return Payload->TryGetStringField(CLAIM_SUBJECT, OutValue);
}

bool FJsonWebToken::GetAudience(FString& OutValue) const
{
	return Payload->TryGetStringField(CLAIM_AUDIENCE, OutValue);
}

bool FJsonWebToken::GetStringClaim(const FString& InName, FString& OutValue) const
{
	return Payload->TryGetStringField(InName, OutValue);
}

bool FJsonWebToken::GetClaim(const FStringView InName, TSharedPtr<FJsonValue>& OutClaim) const
{
	OutClaim = Payload->TryGetField(FString(InName));
	return OutClaim.IsValid();
}


bool FJsonWebToken::HasExpired() const
{
	int64 ExpirationTimestamp = 0;

	if (!GetExpiration(ExpirationTimestamp))
	{
		UE_LOGF(LogJwt, Error, "[FJsonWebToken::HasExpired] Could not get expiration timestamp.");

		return false;
	}

	const FDateTime Now = FDateTime::UtcNow();
	const FDateTime TimeExpires = FDateTime::FromUnixTimestamp(ExpirationTimestamp);

	return Now > TimeExpires;
}


bool FJsonWebToken::Verify() const
{
	UE_LOGF(LogJwt, Error, "[FJsonWebToken::Verify] JWT signature verification is not implemented yet and will always return false.");
	return false;
}


bool FJsonWebToken::Verify(
	const IJwtAlgorithm& Algorithm, const FStringView ExpectedIssuer) const
{
	// Check whether the signature is set
	if (!Signature.IsSet())
	{
		UE_LOGF(LogJwt, Error,
			"[FJsonWebToken::Verify] No signature to verify.");

		return false;
	}

	FString IndicatedAlgorithm;

	// Check whether the algorithm is set
	if (!GetAlgorithm(IndicatedAlgorithm))
	{
		UE_LOGF(LogJwt, Error,
			"[FJsonWebToken::Verify] Could not get token's algorithm.");

		return false;
	}

	// Check whether the algorithms match
	if (Algorithm.GetAlgString() != IndicatedAlgorithm)
	{
		UE_LOGF(LogJwt, Error,
			"[FJsonWebToken::Verify] Algorithms don't match.");

		return false;
	}

	TArray<uint8> EncodedHeaderPayloadBytes;
	FJwtUtils::StringViewToBytes(EncodedHeaderPayload, EncodedHeaderPayloadBytes);

	// Verify the signature
	if (!Algorithm.VerifySignature(EncodedHeaderPayloadBytes, Signature.GetValue()))
	{
		UE_LOGF(LogJwt, Error,
			"[FJsonWebToken::Verify] Signature verification failed.");

		return false;
	}

	FString Issuer;

	// Check whether the issuer is set
	if (!GetIssuer(Issuer))
	{
		UE_LOGF(LogJwt, Error,
			"[FJsonWebToken::Verify] Issuer not set.");

		return false;
	}

	// Check whether the issuers match
	if (ExpectedIssuer != FStringView(Issuer))
	{
		UE_LOGF(LogJwt, Error,
			"[FJsonWebToken::Verify] Issuer does not match expected issuer.");

		return false;
	}

	int64 IssuedAt = 0, Expiration = 0;

	// Check whether IssuedAt and Expiration timestamps are set
	if (!GetIssuedAt(IssuedAt) || !GetExpiration(Expiration))
	{
		UE_LOGF(LogJwt, Error,
			"[FJsonWebToken::Verify] IssuedAt or Expiration timestamp is not set.");

		return false;
	}

	const FDateTime Now = FDateTime::UtcNow();
	const FDateTime TimeIssuedAt = FDateTime::FromUnixTimestamp(IssuedAt);

	// Check whether the token has expired or is invalid
	if (TimeIssuedAt >= Now || HasExpired())
	{
		UE_LOGF(LogJwt, Error,
			"[FJsonWebToken::Verify] Token not valid or has expired already.");

		return false;
	}

	int64 NotBefore = 0;

	// Check whether the token is used before NotBefore, if it's set
	if (GetNotBefore(NotBefore))
	{
		const FDateTime TimeNotBefore = FDateTime::FromUnixTimestamp(NotBefore);

		if (TimeNotBefore >= Now)
		{
			UE_LOGF(LogJwt, Error,
				"[FJsonWebToken::Verify] Token not valid yet.");

			return false;
		}
	}

	return true;
}

