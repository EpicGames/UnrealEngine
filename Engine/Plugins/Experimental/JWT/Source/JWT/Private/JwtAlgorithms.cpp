// Copyright Epic Games, Inc. All Rights Reserved.

#include "JwtAlgorithms.h"


FJwtAlgorithm_RS256::FJwtAlgorithm_RS256()
{
	EncryptionContext = FJwtUtils::GetEncryptionContext();
}


FJwtAlgorithm_RS256::~FJwtAlgorithm_RS256()
{
	if (PublicKey)
	{
		DestroyKey(PublicKey);
	}
}


bool FJwtAlgorithm_RS256::VerifySignature(
	const TArrayView<const uint8> EncodedMessage,
	const TArrayView<const uint8> DecodedSignature) const
{
	if (!EncryptionContext.IsValid())
	{
		return false;
	}

	TArray<uint8> HashedMessage;

	// Hash the encoded message
	if (!EncryptionContext->CalcSHA256(EncodedMessage, HashedMessage))
	{
		return false;
	}

	return EncryptionContext->DigestVerify_RS256(
		HashedMessage, DecodedSignature, PublicKey);
}


bool FJwtAlgorithm_RS256::SetPublicKey(const FStringView InPemKey)
{
	if (!EncryptionContext.IsValid())
	{
		return false;
	}

	if (PublicKey)
	{
		DestroyKey(PublicKey);
	}

	PublicKey = EncryptionContext->GetPublicKey_RSA(InPemKey);

	if (!PublicKey)
	{
		UE_LOGF(LogJwt, Error,
			"[FJwtAlgorithm_RS256::SetPublicKey] "
				"RSA public key from string is invalid.");

		return false;
	}

	return true;
}


void FJwtAlgorithm_RS256::DestroyKey(void* Key)
{
	if (!EncryptionContext.IsValid())
	{
		return;
	}

	EncryptionContext->DestroyKey_RSA(Key);
}

