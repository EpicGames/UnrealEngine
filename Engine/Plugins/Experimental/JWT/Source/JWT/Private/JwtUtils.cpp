// Copyright Epic Games, Inc. All Rights Reserved.

#include "JwtUtils.h"
#include "JwtGlobals.h"

#include "Misc/Base64.h"


TUniquePtr<FEncryptionContext> FJwtUtils::GetEncryptionContext()
{
	TUniquePtr<FEncryptionContext> EncryptionContext
		= IPlatformCrypto::Get().CreateContext();

	if (!EncryptionContext.IsValid())
	{
		UE_LOGF(LogJwt, Error,
			"[JwtUtils::GetEncryptionContext] "
				"EncryptionContext pointer is invalid.");
	}

	return EncryptionContext;
}


bool FJwtUtils::SplitStringView(
	const FStringView InSource, const TCHAR InDelimiter,
	FStringView& OutLeft, FStringView& OutRight)
{
	int32 Position = INDEX_NONE;

	if (!InSource.FindChar(InDelimiter, Position) || Position == INDEX_NONE)
	{
		UE_LOGF(LogJwt, Error,
			"[JwtUtils::SplitStringView] "
				"Could not find delimiter in source string.");

		return false;
	}

	OutLeft = InSource.Left(Position);
	OutRight = InSource.RightChop(Position + 1);

	return true;
}


bool FJwtUtils::SplitEncodedJsonWebTokenString(
	const FStringView InEncodedJsonWebTokenString, FStringView& OutEncodedHeaderPart,
	FStringView& OutEncodedPayloadPart, FStringView& OutEncodedSignaturePart)
{
	FStringView Rest;

	if (!SplitStringView(
		InEncodedJsonWebTokenString, TEXT('.'), OutEncodedHeaderPart, Rest))
	{
		UE_LOGF(LogJwt, Error,
			"[JwtUtils::SplitEncodedJsonWebTokenString] "
				"Cannot extract header from token string.");

		return false;
	}

	if (!SplitStringView(Rest, TEXT('.'), OutEncodedPayloadPart, OutEncodedSignaturePart))
	{
		UE_LOGF(LogJwt, Error,
			"[JwtUtils::SplitEncodedJsonWebTokenString] "
				"Cannot extract payload from token string.");

		return false;
	}

	if (OutEncodedHeaderPart.IsEmpty() || OutEncodedPayloadPart.IsEmpty())
	{
		UE_LOGF(LogJwt, Error,
			"[JwtUtils::SplitEncodedJsonWebTokenString] "
				"Empty header and/or payload in token string.");

		return false;
	}

	return true;
}


bool FJwtUtils::Base64UrlDecode(const FStringView InSource, FString& OutDest)
{
	if (InSource.IsEmpty())
	{
		return false;
	}

	return FBase64::Decode(FString(InSource), OutDest, EBase64Mode::UrlSafe);
}


bool FJwtUtils::Base64UrlDecode(const FStringView InSource, TArray<uint8>& OutDest)
{
	if (InSource.IsEmpty())
	{
		return false;
	}

	return FBase64::Decode(FString(InSource), OutDest, EBase64Mode::UrlSafe);
}


void FJwtUtils::StringViewToBytes(const FStringView InSource, TArray<uint8>& OutBytes)
{
	OutBytes.Reserve(InSource.Len());

	for (const TCHAR& Ch : InSource)
	{
		OutBytes.Add(static_cast<uint8>(Ch));
	}
}
