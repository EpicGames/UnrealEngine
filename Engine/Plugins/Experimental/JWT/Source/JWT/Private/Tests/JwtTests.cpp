// Copyright Epic Games, Inc. All Rights Reserved.

#include "JsonWebToken.h"
#include "JwtAlgorithms.h"

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Containers/StringConv.h"


namespace TestKeys
{

	const FString PUBLICKEY_RSA = "-----BEGIN PUBLIC KEY-----\n"
		"MIIBIjANBgkqhkiG9w0BAQEFAAOCAQ8AMIIBCgKCAQEAmzuYmpOxIBa2n/Iwd+gn"
		"31aOTyg5aHF+63lvet7NOy2KtNYy/jsGtHiSwU6wp/nE52SAXxzBuz8mbAnR8NNO"
		"uSUfDbt7EyO2f3Y1Z8TyNb0n52qcHbLzewtGSUS+YJFcBdkgzstFEIkQTkkLXy2+"
		"b8cvX8EufGxkYe012m4+82WqJPpxaxqOGGHVRyVm92yKQqhsJhgYSKY7ipp1Da0P"
		"wJJe/TBFRsWOvyDoHWSitifkujeb6Xo6qi3owVxCTfKYqCULGnyux4x3O080D6Mh"
		"Tt2ranCl9yBwJTDC81Xd4rlbzW1ttKjB/YC8CDF+YuFlPfZTVZ97BPDz5yCkl/Qz"
		"0wIDAQAB\n"
		"-----END PUBLIC KEY-----";

}

namespace TestJwts
{

	/**
	 * Valid JsonWebToken
	 *
	 * Header: { "typ": "JWT", "alg": "RS256" }
	 * Payload: { "iss": "epicgames.com", "iat": 1687528538,
	 *			  "exp": 2003140949, "nbf": 1687529044 }
	 */
	const FString JWT_RS256_VALID =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiaWF0IjoxNjg3NTI4NTM4LCJleHAiOjIwMDM"
		"xNDA5NDksIm5iZiI6MTY4NzUyOTA0NH0.PsUElqoe4WNOFKmYEOFPbXkXcDLgJ8qnQV"
		"n6rzpjMwZPcVGOP9oIuoENVVxfpSZ2Y5qjd2tE53AqkeyroNiRMobP6K8auotw4_10V"
		"cb2MK2yt7K6VtL-O_Ne0lzQzKUY6K5WStc0q6u2jFWE5NjJeBkE2NQDA_rpoWitv46o"
		"nIWeqkvxQCtDlSKcl_VYnhPs_ON1IWJY7jCDjFgMPcbKcjDquMCceY-RMPgJj8ly_Vz"
		"2Oqfzu3PNRG6zc8PbMVwVRlhG_QxYlc0d8goa_PQoI-2P52UiCm5091Lrr-vzJPVnMU"
		"m_NXaak7ahK4Wb6nYsFcdFAPuSPh5ePh1qdkPEDw";

	// Jwt has no signature, but a second period
	const FString JWT_RS256_NO_SIGNATURE =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiaWF0IjoxNjg3NTI4NTM4LCJleHAiOjIwMDM"
		"xNDA5NDksIm5iZiI6MTY4NzUyOTA0NH0.";

	// Jwt has algorithm "none"
	const FString JWT_RS256_NONE_ALGORITHM =
		"eyJhbGciOiJub25lIiwidHlwIjoiSldUIn0."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiaWF0IjoxNjg3NTI4NTM4LCJleHAiOjIwMDM"
		"xNDA5NDksIm5iZiI6MTY4NzUyOTA0NH0.PsUElqoe4WNOFKmYEOFPbXkXcDLgJ8qnQV"
		"n6rzpjMwZPcVGOP9oIuoENVVxfpSZ2Y5qjd2tE53AqkeyroNiRMobP6K8auotw4_10V"
		"cb2MK2yt7K6VtL-O_Ne0lzQzKUY6K5WStc0q6u2jFWE5NjJeBkE2NQDA_rpoWitv46o"
		"nIWeqkvxQCtDlSKcl_VYnhPs_ON1IWJY7jCDjFgMPcbKcjDquMCceY-RMPgJj8ly_Vz"
		"2Oqfzu3PNRG6zc8PbMVwVRlhG_QxYlc0d8goa_PQoI-2P52UiCm5091Lrr-vzJPVnMU"
		"m_NXaak7ahK4Wb6nYsFcdFAPuSPh5ePh1qdkPEDw";

	// Jwt signature is wrong
	const FString JWT_RS256_WRONG_SIGNATURE =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiaWF0IjoxNjg3NTI4NTM4LCJleHAiOjIwMDM"
		"xNDA5NDksIm5iZiI6MTY4NzUyOTA0NH0.PsUElqoe4WNOFKmYEOFPbXkXcDLgJ8qnQV"
		"n6rzpjMwZPcVGOP9oIuoENVVxfpSZ2Y5qjd2tE53AqkeyroNiRMobP6K8auotw4_10V"
		"cb2MK2yt7K6VtL-O_Ne0QzzlKUY6K5WStc0q6u2jFWE5NjJeBkE2NQDA_rpoWitv46o"
		"nIWeqkvxQCtDlSKcl_VYnhPs_ON1IWJY7jCDjFgMPcbKcjDquMCceY-RMPgJj8ly_Vz"
		"2Oqfzu3PNRG6zc8PbMVwVRlhG_QxYlc0d8goa_PQoI-2P52UiCm5091Lrr-vzJPVnMU"
		"m_NXaak7ahK4Wb6nYsFcdFAPuSPh5ePh1qdkwDEP";

	// Jwt doesn't have an issuer specified
	const FString JWT_RS256_NO_ISSUER =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpYXQiOjE2ODc1Mjg1MzgsImV4cCI6MjAwMzE0MDk0OSwibmJmIjoxNjg3NTI5MDQ"
		"0fQ.iCcyrMsg4ufaeACbPbxEQDG7KPqS38Ekwhv7CatyMET5S6T7Pu1mV1gO7MSpyed"
		"mRpdlypCyQaFDV1J9kFRckN12MuRV4HPbnIZ0Jeca35mdy_GNTKo_iydsMUSZupo_K1"
		"8uYKlfMW05hwsy0J5xfdJzuCqYDdNeabFO5YiwOGCsRcCqMWQL_y_12Tlwy63tKCFj7"
		"jVuwwYSC4rgKSvLhM_dYICf4S7It2YjRpcQLEEWTaKp67Ywx1AUpatzjII1wUwyKChz"
		"9jDGxq88jWZcfyLrDmEBtFL4Rq76qVIFJqe3D9ZbaXs57-wQtk83sc-E9_fcE4ESMor"
		"0jBDURkx4uA";

	// Jwt has a wrong issuer specified
	const FString JWT_RS256_WRONG_ISSUER =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJteS1lcGljZ2FtZXMuY29tIiwiaWF0IjoxNjg3NTI4NTM4LCJleHAiOjI"
		"wMDMxNDA5NDksIm5iZiI6MTY4NzUyOTA0NH0.ZGGg2akgL9vBynRiAfJmS1LEgmDpC5"
		"4FS8kcPEB1z0yr1p8lFCRyGSD4iBD-cpweqfYq_4CsOV9Hq8frj9xwlQJEFBmBsk1Ff"
		"XCsKEwXnCi76bAkL0BwqsaDbqHnNo_eI1_cw45pWNMVJdshO8hadP6mMRIgRqZHebud"
		"ifP4zkVGYBwjgybHyk_hv1qbyRg3_UOozGgdcd7LvwLdmdKwFQT-gEoPxIhxZLD9-xm"
		"M1OSWYITBV5_AUrmQH_rPcXiZU8Gt0nIC7LpIaFGdu1M3GIPMw510C9M0sG3zxAYYv0"
		"kN9y0tN0AZsWEAf8nt0mXkcfFdc33NmbnVAAN-m8ndTA";

	// Jwt doesn't have a issuedat specified
	const FString JWT_RS256_NO_ISSUEDAT =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiZXhwIjoyMDAzMTQwOTQ5LCJuYmYiOjE2ODc"
		"1MjkwNDR9.dBJ3JaUc3_SCsGq9I4f7FT-OrNmKbP1KQUWz1bD0H6x2_1L6IArCaAeRh"
		"UOCuN2bjoIP9SguTMY5mz5jqSwDXfcaFjhPG5qbAp7ieU4ljgb5FLJlZTpSvLxcJ9YR"
		"dUUBGo_-sw2WhlVxcVH_hme6CCc42oRoTcBeWaw8JG6KSbVFEPxTwLVaED6UtE7ra6A"
		"k-YjjKbcaZCqKm8xe2_wRAL2hDJprNM677CFOXUTDGXpClKy1cEPNlIlsS24S5lTxGa"
		"_-qq3IgI0cy0GRyHDJ9g6bBrb9YL6biFGYoyXP6fis0wGoIAJJZ3q3AhS0nPIJipokg"
		"IkrcPN5FveEHRR2VA";

	// Jwt has a future issuedat specified
	const FString JWT_RS256_FUTURE_ISSUEDAT =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiaWF0IjoyMDAzMTQwOTQ5LCJleHAiOjIwMDM"
		"xNDA5NDksIm5iZiI6MTY4NzUyOTA0NH0.Muo4Yh4BEVygyp6qgH4D76sTpXoW_3KHyx"
		"I_3fm2GiWI34T4SOQXT6TJGVNS_aTd1Dkhz_UPJU1LOlBzAyK4erJmBu-VGTKESGeeG"
		"V2fP6FTZ-zAAAuQkUb44IxIC_o_2BlCyTFi1qQyLVWMOXaN3s9sJmduvrUKZCWEBEQk"
		"T1aHoGapYVX3OjC8P31JJP4_afswi4M3iTCuWlPydaaUhzDhp2sgcpbgJpq2ZaWxO2Z"
		"QUVqRg3qwgM_V-VHSMsGvCOhN_-adA0ji6_5zpI7wlTbDCaRhgop5MDdk4Sa_7gM2VV"
		"tepnET7t-YH-aTyAkcvUFBw2GZUrLzNQaTk15nyA";

	// Jwt doesn't have a expiration specified
	const FString JWT_RS256_NO_EXPIRATION =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiaWF0IjoxNjg3NTI4NTM4LCJuYmYiOjE2ODc"
		"1MjkwNDR9.CGkMQjeuD9YKX6zAlRFGj4tTq63im3pZHBCk-l-VrpUdKWv8WEVt64VDe"
		"yWqFGzfMO26OGRUS-pII9U76ZcJiAKWRdzp_zWl_WsieWbBMQziSwLQBvbsLmkUdsHc"
		"sitYFTXdvugVbwNCyUpsdzvfsH3L3RvUkBSn1ZcSiCWPYFDtC1tURrYYMtH5JixrMr4"
		"dap57s5bfWjv_1pVW4ubpUUrq19l9BSNyw_MA5nfwFyzzPEAylY1ND9NyTPPHP0ure8"
		"Br4gZWYEoLrYY6T8YfZf0lMayaGqCIcVtymYp7syI_Nl0JAN7C__a4xcrVpTgsazGcJ"
		"sF7L06KTxdmbsR-Hw";

	// Jwt has a past expiration specified
	const FString JWT_RS256_PAST_EXPIRATION =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiaWF0IjoxNjg3NTI4NTM4LCJleHAiOjE2ODc"
		"1Mjg1MzgsIm5iZiI6MTY4NzUyOTA0NH0.b1yDNS6PDbg-z1XZ3h41YdvJwvXFMzun0a"
		"AuG9RvrXzp3B4UoHVi8Wzkwhjq1T-CVKcaUgpHcX50MymsrXFmuBJpflyFzfa_CwBg0"
		"iL9AdSIJODO0UGi07wbZfx71rib_-WL-SVjeS_iQ8ftvPzDz3AyKG_Cd-9eWq3stD2O"
		"HlZ-kqiy9Mn8BuopYbNY7K2e936YR5uHSu9JOXkbMIlHaVZxR5MBAShD6fb4a41lXoA"
		"srtrKpim9T2Hg8xaZnXl1buuwQR-fRGLpPndnRfXvA1xhwLdSvSkbw0V71z4GKxSW0b"
		"ZGQMT7tmL1ft8gDamRuIT6OVqDwa7Dh2DQ2pgvfw";

	// Jwt has a future before specified
	const FString JWT_RS256_FUTURE_NOTBEFORE =
		"eyJ0eXAiOiJKV1QiLCJhbGciOiJSUzI1NiJ9."
		"eyJpc3MiOiJlcGljZ2FtZXMuY29tIiwiaWF0IjoxNjg3NTI4NTM4LCJleHAiOjIwMDM"
		"xNDA5NDksIm5iZiI6MjAwMzE0MDk0OX0.Qe81jAxSvG2grEH5T34SpHwqsrjEnqKn9R"
		"jHrC7qIGfakxJtU8AjJ8t6oLSWqDJLmaZPYZmi0qC1ziMcx52a-e5mVI9r_QH2THyBK"
		"DCKQInnak0QO1_c3AlNudrGYehHcqW-nCsuEs7gOVW0FqVbe_b4L0uVQcYuydX-LCdO"
		"3XnF2VDEGgjiAsXHZx847EpAllJ6neXzTdOAzyeH4d8NcKl8FxoXxl-bj4tP5wSvEb-"
		"guxo6Ki59_EQSbX0INLJ3eDKcZEMPMZssTEPmqF6o7xVGFTfWuVKBzP0Bmqw2ccNLuD"
		"cidlbpZY8Imoe22P1WRyB8cjgJv0fbC76S90YpJA";

}


/**
 * JWT verification test
 *
 * This JWT verification test covers all possible control paths of
 * the JsonWebToken::Verify() method by parsing a manually crafted
 * JWT for each scenario. Only one of the sub-tests, the first one,
 * is expected to succeed, while the rest is expected to fail.
 * The verification test fails in case any of the sub-tests do not
 * fail but were expected to fail.
 */
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FJwtVerificationTest, "JWT.Verification",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FJwtVerificationTest::RunTest(const FString& Parameters)
{
	FJwtAlgorithm_RS256 Verifier;

	AddInfo(TEXT("Starting the JWT verification test ..."));

	// Set the public key
	{
		if (!Verifier.SetPublicKey(TestKeys::PUBLICKEY_RSA))
		{
			AddError(TEXT("Could not set public key."));
			return false;
		}
	}

	const FString Issuer = "epicgames.com";

	AddInfo(TEXT("Public key was set."));

	// Verify valid Jwt
	{
		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_VALID);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_VALID."));
			return false;
		}

		if (!Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Could not verify JWT_RS256_VALID."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully verified JWT_RS256_VALID."));

	// Fail to verify Jwt without a signature
	{
		AddExpectedError(TEXT("No signature to verify"));

		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_NO_SIGNATURE);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_NO_SIGNATURE."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_NO_SIGNATURE."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_NO_SIGNATURE."));

	// Fail to verify Jwt with none algorithm
	{
		AddExpectedError(TEXT("Algorithms don't match"));

		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_NONE_ALGORITHM);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_NONE_ALGORITHM."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_NONE_ALGORITHM."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_NONE_ALGORITHM."));

	// Fail to verify Jwt with wrong signature
	{
		AddExpectedError(TEXT("Signature verification failed"));
		AddExpectedError(TEXT("Signature is invalid"));

		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_WRONG_SIGNATURE);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_WRONG_SIGNATURE."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_WRONG_SIGNATURE."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_WRONG_SIGNATURE."));

	// Fail to verify Jwt without an issuer
	{
		AddExpectedError(TEXT("Issuer not set"));

		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_NO_ISSUER);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_NO_ISSUER."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_NO_ISSUER."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_NO_ISSUER."));

	// Fail to verify Jwt with a wrong issuer
	{
		AddExpectedError(TEXT("Issuer does not match expected issuer"));

		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_WRONG_ISSUER);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_WRONG_ISSUER."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_WRONG_ISSUER."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_WRONG_ISSUER."));

	// Add two occurrences of expected errors for the next two pairs of tests
	{
		AddExpectedError(TEXT("Token not valid or has expired already"),
			EAutomationExpectedErrorFlags::Contains, 2);

		AddExpectedError(TEXT("IssuedAt or Expiration timestamp is not set"),
			EAutomationExpectedErrorFlags::Contains, 2);
	}

	// Fail to verify Jwt without an issuedat
	{
		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_NO_ISSUEDAT);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_NO_ISSUEDAT."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_NO_ISSUEDAT."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_NO_ISSUEDAT."));

	// Fail to verify Jwt with a future issuedat
	{
		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_FUTURE_ISSUEDAT);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_FUTURE_ISSUEDAT."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_FUTURE_ISSUEDAT."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_FUTURE_ISSUEDAT."));

	// Fail to verify Jwt without an expiration
	{
		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_NO_EXPIRATION);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_NO_EXPIRATION."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_NO_EXPIRATION."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_NO_EXPIRATION."));

	// Fail to verify Jwt with a past expiration
	{
		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_PAST_EXPIRATION);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_PAST_EXPIRATION."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_PAST_EXPIRATION."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_PAST_EXPIRATION."));

	// Fail to verify Jwt with a future notbefore
	{
		AddExpectedError(TEXT("Token not valid yet"));

		TOptional<FJsonWebToken> Token
			= FJsonWebToken::FromString(TestJwts::JWT_RS256_FUTURE_NOTBEFORE);

		if (!Token.IsSet())
		{
			AddError(TEXT("Could not parse JWT in JWT_RS256_FUTURE_NOTBEFORE."));
			return false;
		}

		if (Token->Verify(Verifier, Issuer))
		{
			AddError(TEXT("Verified invalid JWT_RS256_FUTURE_NOTBEFORE."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to verify JWT_RS256_FUTURE_NOTBEFORE."));

	// Fail to parse empty Jwt
	{
		TOptional<FJsonWebToken> Token = FJsonWebToken::FromString(FString(""));

		if (Token.IsSet())
		{
			AddError(TEXT("Empty JWT was parsed."));
			return false;
		}
	}

	AddInfo(TEXT("Successfully failed to parse empty JWT."));

	AddInfo(TEXT("Successfully completed JWT verification test."));

	return true;
}
