// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Dom/JsonObject.h" // IWYU pragma: keep

#define UE_API JWT_API

class FJsonObject;
class FJsonValue;
class IJwtAlgorithm;
enum class EJson;


class FJsonWebToken
{
public:
	/**
	 * Creates a JWT from the provided string.
	 * The string must consist of 3 base64 url encoded parts: a header, payload, and signature.
	 * The parts must be split by a period character.
	 * The signature part is optional. If the signature is excluded, the string must still contain a period character in its place.
	 * Valid formats: "header.payload.signature" and "header.payload."
	 *
	 * @param InJsonWebTokenString The string to decode.
	 * @return An optional set with a FJsonWebToken if the JWT was successfully decoded.
	 */
	static UE_API TOptional<FJsonWebToken> FromString(const FStringView InEncodedJsonWebToken);

	static UE_API bool FromString(const FStringView InEncodedJsonWebToken, FJsonWebToken& OutJsonWebToken);

	/**
	 * Get raw JSON object for payload, allowing for custom claim parsing with FJsonSerializable classes.
	 * 
	 * @return reference to FJsonObject for the payload.
	 */
	const TSharedRef<FJsonObject>& GetPayload() const { return Payload; }

	/**
	 * Gets the type.
	 *
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetType(FString& OutValue) const;

	/**
	 * Gets the key id.
	 *
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetKeyId(FString& OutValue) const;

	/**
	 * Gets the algorithm that was used to construct the signature.
	 *
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetAlgorithm(FString& OutValue) const;

	/**
	 * Gets the issuer domain.
	 * 
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetIssuer(FString& OutValue) const;

	/**
	 * Gets the issued at timestamp.
	 *
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetIssuedAt(int64& OutValue) const;

	/**
	 * Gets the expiration timestamp.
	 *
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetExpiration(int64& OutValue) const;

	/**
	 * Gets the not valid before timestamp.
	 *
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetNotBefore(int64& OutValue) const;

	/**
	 * Gets the subject.
	 *
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetSubject(FString& OutValue) const;

	/**
	 * Gets the audience.
	 *
	 * @param OutValue The value to output on success.
	 * @return Whether the value exists and was successfully outputted.
	 */
	UE_API bool GetAudience(FString& OutValue) const;

	/**
	 * Get a custom claim by name.
	 * 
	 * @param InName The name of the claim.
	 * @param OutValue The value to output on success.
	 * @return Whether the claim exists and was successfully outputted.
	 */
	UE_API bool GetStringClaim(const FString& InName, FString& OutValue) const;

	/**
	 * Gets a claim by name.
	 * This method can be used to get custom claims that are not reserved as part of the JWT specification.
	 *
	 * @param InName The name of the claim.
	 * @param OutClaim The json value to output on success.
	 * @return Whether the claim exists and was successfully outputted.
	 */
	UE_API bool GetClaim(const FStringView InName, TSharedPtr<FJsonValue>& OutClaim) const;

	/**
	 * Gets a claim by name.
	 * This method can be used to get custom claims that are not reserved as part of the JWT specification.
	 *
	 * @param InName The name of the claim.
	 * @return The json value to output on success, or an invalid shared pointer on failure.
	 */
	template<EJson JsonType>
	TSharedPtr<FJsonValue> GetClaim(const FStringView InName) const
	{
		return Payload->GetField<JsonType>(FString(InName));
	}

	/**
	 * Checks whether the tokens expiration timestamp is in the past.
	 *
	 * @return Whether the token has expired
	 */
	UE_API bool HasExpired() const;

	/**
	 * Deprecated method to signature validate the JWT.
	 *
	 * @return False
	 */
	UE_DEPRECATED(5.3, "Verify() without arguments is deprecated. Please use Verify(Algorithm, ExpectedIssuer) instead.")
	UE_API bool Verify() const;

	/**
	 * Signature validate and verify the JWT.
	 * - Validates the signature against the encoded header and encoded payload
	 * - Verifies the basic claims of the JWT
	 * - Ensures the issuers match
	 *
	 * @param Algorithm Implementation of the cryptographic algorithm used for signature validation
	 * @param ExpectedIssuer The expected issuer
	 *
	 * @return Whether the JWT was successfully verified
	 */
	UE_API bool Verify(
		const IJwtAlgorithm& Algorithm, const FStringView ExpectedIssuer) const;

private:
	UE_API FJsonWebToken(
		const FStringView InEncodedHeaderPayload, const TSharedRef<FJsonObject>& InHeaderPtr,
		const TSharedRef<FJsonObject>& InPayloadPtr, const TOptional<TArray<uint8>>& InSignature);

	static UE_API void DumpJsonObject(const FJsonObject& InJsonObject);

	static UE_API TSharedPtr<FJsonObject> FromJson(const FString& InJsonStr);

	static UE_API TSharedPtr<FJsonObject> ParseEncodedJson(const FStringView InEncodedJson);

public:
	// JWT payload registered claim field names
	static UE_API const TCHAR *const CLAIM_ISSUER;
	static UE_API const TCHAR *const CLAIM_ISSUED_AT;
	static UE_API const TCHAR *const CLAIM_EXPIRATION;
	static UE_API const TCHAR* const CLAIM_NOT_BEFORE;
	static UE_API const TCHAR *const CLAIM_SUBJECT;
	static UE_API const TCHAR *const CLAIM_AUDIENCE;

	// JWT header field names
	static UE_API const TCHAR* const HEADER_TYPE;
	static UE_API const TCHAR* const HEADER_KEY_ID;
	static UE_API const TCHAR* const HEADER_ALGORITHM;

	// JWT header expected values
	static UE_API const TCHAR* const TYPE_VALUE_JWT;

private:

	/** The encoded header and payload parts. */
	FString EncodedHeaderPayload;

	/** The decoded and parsed header. */
	TSharedRef<FJsonObject> Header;

	/** The decoded and parsed payload. */
	TSharedRef<FJsonObject> Payload;

	/** The decoded signature. */
	TOptional<TArray<uint8>> Signature;
};

#undef UE_API
