// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JwtGlobals.h"

#include "CoreMinimal.h"


class IJwtAlgorithm
{

public:

	virtual ~IJwtAlgorithm() = default;

public:

	/**
	 * Gets the algorithm's string representation.
	 */
	virtual inline const FString& GetAlgString() const = 0;

	/**
	 * Verifies the decoded signature with the encoded message.
	 * 
	 * @param EncodedMessage The base64 encoded message {header}.{payload}
	 * @param DecodedSignature The decoded JWT signature
	 * 
	 * @return Whether the signature was successfully verified.
	 */
	virtual bool VerifySignature(
		const TArrayView<const uint8> EncodedMessage,
		const TArrayView<const uint8> DecodedSignature) const = 0;

};
