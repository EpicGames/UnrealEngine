// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "JwtAlgorithm.h"

#include "JwtUtils.h"

#define UE_API JWT_API


class FJwtAlgorithm_RS256
	: public IJwtAlgorithm
{

public:

	UE_API FJwtAlgorithm_RS256();

	FJwtAlgorithm_RS256(const FJwtAlgorithm_RS256&) = delete;

	FJwtAlgorithm_RS256& operator=(const FJwtAlgorithm_RS256&) = delete;

	UE_API ~FJwtAlgorithm_RS256();

public:

	// ~ IJwtAlgorithm interface

	virtual inline const FString& GetAlgString() const override
	{
		return AlgorithmString;
	}

	UE_API virtual bool VerifySignature(
		const TArrayView<const uint8> EncodedMessage,
		const TArrayView<const uint8> DecodedSignature) const override;

public:

	/**
	 * Set the public RSA key from PEM.
	 *
	 * @param InPemKey Key in PEM format as string view
	 *
	 * @return Whether the operation was successful
	 */
	UE_API bool SetPublicKey(const FStringView InPemKey);

private:

	/**
	 * Free the RSA key.
	 *
	 * @param Key The key pointer
	 */
	UE_API void DestroyKey(void* Key);

protected:

	/** Holds the algorithm string. */
	const FString AlgorithmString = "RS256";

private:

	/** Holds the unique pointer to the encryption context (OpenSSL/SwitchSSL). */
	TUniquePtr<FEncryptionContext> EncryptionContext = nullptr;

	/** Holds the public key pointer. */
	void* PublicKey = nullptr;

};

#undef UE_API
