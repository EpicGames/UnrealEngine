// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IPlatformCrypto.h"


class FJwtUtils
{

public:

	/**
	 * Get the pointer to the current platform
	 * encryption context (e.g. OpenSSL or SwitchSSL).
	 *
	 * @return Pointer to the encryption context
	 */
	static TUniquePtr<FEncryptionContext> GetEncryptionContext();

	/**
	 * Split a StringView on a delimiter.
	 *
	 * @param InSource The source string view
	 * @param InDelimiter The delimiter
	 * @param OutLeft Out string view for the left part
	 * @param OutRight Out string view for the right part
	 *
	 * @return Whether the string view could be split
	 */
	static bool SplitStringView(
		const FStringView InSource, const TCHAR InDelimiter,
		FStringView& OutLeft, FStringView& OutRight);

	/**
	 * Split an encoded JWT string into the 3 header, payload, and signature parts.
	 *
	 * @param InEncodedJsonWebTokenString The JWT string
	 * @param OutEncodedHeaderPart Out string for the header part
	 * @param OutEncodedPayloadPart Out string for the payload part
	 * @param OutEncodedSignaturePart Out string for the signature part
	 *
	 * @return Whether the JWT string could be split
	 */
	static bool SplitEncodedJsonWebTokenString(
		const FStringView InEncodedJsonWebTokenString, FStringView& OutEncodedHeaderPart,
		FStringView& OutEncodedPayloadPart, FStringView& OutEncodedSignaturePart);

	/**
	 * Decode base64 in url-safe mode from string to string.
	 *
	 * @param InSource The source string
	 * @param OutDest The destination string
	 *
	 * @return Whether the string could be decoded
	 */
	static bool Base64UrlDecode(const FStringView InSource, FString& OutDest);

	/**
	 * Decode base64 in url-safe mode from string to byte array.
	 *
	 * @param InSource The source string
	 * @param OutDest The destination byte array
	 *
	 * @return Whether the string could be decoded
	 */
	static bool Base64UrlDecode(const FStringView InSource, TArray<uint8>& OutDest);

	/**
	 * Convert a StringView to a byte array.
	 *
	 * @param InSource The source string
	 * @param OutBytes The destination byte array
	 */
	static void StringViewToBytes(const FStringView InSource, TArray<uint8>& OutBytes);

};
